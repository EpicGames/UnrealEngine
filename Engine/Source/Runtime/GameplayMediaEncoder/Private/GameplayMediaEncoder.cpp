// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayMediaEncoder.h"
#include "Engine/GameEngine.h"
#include "HAL/IConsoleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "RendererInterface.h"
#include "ScreenRendering.h"
#include "ShaderCore.h"
#include "PipelineStateCache.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "IbmLiveStreaming.h"
#include "Slate/SlateViewportProvider.h"

#include "VideoEncoderFactory.h"
#include "VideoEncoderInput.h"

#include "ClearQuad.h"
#include "CommonRenderResources.h"

#include "AudioEncoderFactory.h"

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
#include "ID3D11DynamicRHI.h"
#include "ID3D12DynamicRHI.h"
#include "IVulkanDynamicRHI.h"
#endif

DEFINE_LOG_CATEGORY(GameplayMediaEncoder);
CSV_DEFINE_CATEGORY(GameplayMediaEncoder, true);

// right now we support only 48KHz audio sample rate as it's the only config UE4 seems to output
// WMF AAC encoder supports also 44100Hz so its support can be easily added
const uint32 HardcodedAudioSamplerate = 48000;
// for now we downsample to stereo. WMF AAC encoder also supports 6 (5.1) channels
// so it can be added too
const uint32 HardcodedAudioNumChannels = 2;
// currently neither IVideoRecordingSystem neither HighlightFeature APIs allow to configure
// audio stream parameters
const uint32 HardcodedAudioBitrate = 192000;

// currently neither IVideoRecordingSystem neither HighlightFeature APIs allow to configure
// video stream parameters
#if PLATFORM_WINDOWS
const uint32 HardcodedVideoFPS = 60;
#else
const uint32 HardcodedVideoFPS = 30;
#endif
const uint32 HardcodedVideoBitrate = 20000000;
const uint32 MinVideoBitrate = 1000000;
const uint32 MaxVideoBitrate = 20000000;
const uint32 MinVideoFPS = 10;
const uint32 MaxVideoFPS = 60;

const uint32 MaxWidth = 1920;
const uint32 MaxHeight = 1080;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FAutoConsoleCommand GameplayMediaEncoderInitialize(TEXT("GameplayMediaEncoder.Initialize"), TEXT("Constructs the audio/video encoding objects. Does not start encoding"), FConsoleCommandDelegate::CreateStatic(&FGameplayMediaEncoder::InitializeCmd));

FAutoConsoleCommand GameplayMediaEncoderStart(TEXT("GameplayMediaEncoder.Start"), TEXT("Starts encoding"), FConsoleCommandDelegate::CreateStatic(&FGameplayMediaEncoder::StartCmd));

FAutoConsoleCommand GameplayMediaEncoderStop(TEXT("GameplayMediaEncoder.Stop"), TEXT("Stops encoding"), FConsoleCommandDelegate::CreateStatic(&FGameplayMediaEncoder::StopCmd));

FAutoConsoleCommand GameplayMediaEncoderShutdown(TEXT("GameplayMediaEncoder.Shutdown"), TEXT("Releases all systems."), FConsoleCommandDelegate::CreateStatic(&FGameplayMediaEncoder::ShutdownCmd));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//////////////////////////////////////////////////////////////////////////
//
// FGameplayMediaEncoder
//
//////////////////////////////////////////////////////////////////////////

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedPtr<FGameplayMediaEncoder, ESPMode::ThreadSafe> FGameplayMediaEncoder::Singleton = { };
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGameplayMediaEncoder* FGameplayMediaEncoder::Get()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	// Constructed and captured as a thread safe shared pointer as this is required by the audio submix listener interface
	if(!Singleton.IsValid())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Singleton = MakeShared<FGameplayMediaEncoder>();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	return Singleton.Get();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGameplayMediaEncoder::FGameplayMediaEncoder() {}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGameplayMediaEncoder::~FGameplayMediaEncoder()
{
	bIsInDestructor = true;
	Shutdown();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FGameplayMediaEncoder::RegisterListener(IGameplayMediaEncoderListener* Listener)
{
	check(IsInGameThread());

	ListenersCS.Lock();
	bool bFirstListener = Listeners.Num() == 0;
	ListenersCS.Unlock();
	
	if(bFirstListener)
	{
		UE_LOGF(GameplayMediaEncoder, Log, "Registering the first listener");
		if(!Start())
		{
			return false;
		}
	}

	FScopeLock Lock(&ListenersCS);
	Listeners.AddUnique(Listener);
	return true;
}

void FGameplayMediaEncoder::UnregisterListener(IGameplayMediaEncoderListener* Listener)
{
	check(IsInGameThread());

	ListenersCS.Lock();
	Listeners.Remove(Listener);
	bool bAnyListenersLeft = Listeners.Num() > 0;
	ListenersCS.Unlock();

	if(bAnyListenersLeft == false)
	{
		UE_LOGF(GameplayMediaEncoder, Log, "Unregistered the last listener");
		Stop();
	}
}

bool FGameplayMediaEncoder::Initialize()
{
	MemoryCheckpoint("Initial");

	if(VideoEncoder)
	{
		UE_LOGF(GameplayMediaEncoder, Log, "Already initialized");
		return true;
	}

	// If some error occurs, call Shutdown to cleanup
	bool bIsOk = false;
	ON_SCOPE_EXIT
	{
		if(!bIsOk)
		{
			Shutdown();
		}
	};

	//
	// Audio
	//
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AVEncoder::FAudioEncoderFactory* AudioEncoderFactory = AVEncoder::FAudioEncoderFactory::FindFactory("aac");
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if(!AudioEncoderFactory)
	{
		UE_LOGF(GameplayMediaEncoder, Error, "No audio encoder for aac found");
		return false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AudioEncoder = AudioEncoderFactory->CreateEncoder("aac");
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if(!AudioEncoder)
	{
		UE_LOGF(GameplayMediaEncoder, Error, "Could not create audio encoder");
		return false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AVEncoder::FAudioConfig AudioConfig;
	AudioConfig.Samplerate = HardcodedAudioSamplerate;
	AudioConfig.NumChannels = HardcodedAudioNumChannels;
	AudioConfig.Bitrate = HardcodedAudioBitrate;
	if(!AudioEncoder->Initialize(AudioConfig))
	{
		UE_LOGF(GameplayMediaEncoder, Error, "Could not initialize audio encoder");
		return false;
	}

	AudioEncoder->RegisterListener(*this);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	MemoryCheckpoint("Audio encoder initialized");

	//
	// Video
	//
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	VideoConfig.Codec = "h264";
	VideoConfig.Height = VideoConfig.Width = VideoConfig.Framerate = VideoConfig.Bitrate = 0;
	FParse::Value(FCommandLine::Get(), TEXT("GameplayMediaEncoder.ResY="), VideoConfig.Height);
	UE_LOGF(GameplayMediaEncoder, Log, "GameplayMediaEncoder.ResY = %d", VideoConfig.Height);
	if(VideoConfig.Height == 0 || VideoConfig.Height == 720)
	{
		VideoConfig.Width = 1280;
		VideoConfig.Height = 720;
	}
	else if(VideoConfig.Height == 1080)
	{
		VideoConfig.Width = 1920;
		VideoConfig.Height = 1080;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	else
	{
		UE_LOGF(GameplayMediaEncoder, Fatal, "GameplayMediaEncoder.ResY can only have a value of 720 or 1080");
		return false;
	}

	// Specifying 0 will completely disable frame skipping (therefore encoding as many frames as possible)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FParse::Value(FCommandLine::Get(), TEXT("GameplayMediaEncoder.FPS="), VideoConfig.Framerate);
	if(VideoConfig.Framerate == 0)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Note : When disabling frame skipping, we lie to the encoder when initializing.
		// We still specify a framerate, but then feed frames without skipping
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		VideoConfig.Framerate = HardcodedVideoFPS;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bDoFrameSkipping = false;
		UE_LOGF(GameplayMediaEncoder, Log, "Uncapping FPS");
	}
	else
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		VideoConfig.Framerate = FMath::Clamp(VideoConfig.Framerate, (uint32)MinVideoFPS, (uint32)MaxVideoFPS);
		UE_LOGF(GameplayMediaEncoder, Log, "Capping FPS %u", VideoConfig.Framerate);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bDoFrameSkipping = true;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	VideoConfig.Bitrate = HardcodedVideoBitrate;
	FParse::Value(FCommandLine::Get(), TEXT("GameplayMediaEncoder.Bitrate="), VideoConfig.Bitrate);
	VideoConfig.Bitrate = FMath::Clamp(VideoConfig.Bitrate, (uint32)MinVideoBitrate, (uint32)MaxVideoBitrate);

	AVEncoder::FVideoEncoder::FLayerConfig videoInit;
	videoInit.Width = VideoConfig.Width;
	videoInit.Height = VideoConfig.Height;
	videoInit.MaxBitrate = MaxVideoBitrate;
	videoInit.TargetBitrate = VideoConfig.Bitrate;
	videoInit.MaxFramerate = VideoConfig.Framerate;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if(GDynamicRHI)
	{
		const ERHIInterfaceType RHIType = RHIGetInterfaceType();

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
		if (RHIType == ERHIInterfaceType::D3D11)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			VideoEncoderInput = AVEncoder::FVideoEncoderInput::CreateForD3D11(GetID3D11DynamicRHI()->RHIGetDevice(), true, IsRHIDeviceAMD());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else if (RHIType == ERHIInterfaceType::D3D12)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			VideoEncoderInput = AVEncoder::FVideoEncoderInput::CreateForD3D12(GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU(), true, IsRHIDeviceNVIDIA());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else if (RHIType == ERHIInterfaceType::Vulkan)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			VideoEncoderInput = AVEncoder::FVideoEncoderInput::CreateForVulkan(GetIVulkanDynamicRHI()->RHIGetVkDevice(), true);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
#endif
		{
			UE_LOGF(GameplayMediaEncoder, Error, "Video encoding is not supported with the current Platform/RHI combo.");
			return false;
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<AVEncoder::FVideoEncoderInfo>& AvailableEncodersInfo = AVEncoder::FVideoEncoderFactory::Get().GetAvailable();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (AvailableEncodersInfo.Num() == 0)
	{
		UE_LOGF(GameplayMediaEncoder, Error, "No video encoders found. Check if relevent encoder plugins have been enabled for this project.");
		return false;
	}

	for (const auto& EncoderInfo : AvailableEncodersInfo)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (EncoderInfo.CodecType == AVEncoder::ECodecType::H264)
		{
			VideoEncoder = AVEncoder::FVideoEncoderFactory::Get().Create(EncoderInfo.ID, VideoEncoderInput, videoInit);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (!VideoEncoder)
	{
		UE_LOGF(GameplayMediaEncoder, Error, "No H264 video encoder found. Check if relevent encoder plugins have been enabled for this project.");
		return false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	VideoEncoder->SetOnEncodedPacket([this](uint32 LayerIndex, const TSharedPtr<AVEncoder::FVideoEncoderInputFrame> Frame, const AVEncoder::FCodecPacket& Packet)
	                                 { OnEncodedVideoFrame(LayerIndex, Frame, Packet); });
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if(!VideoEncoder)
	{
		UE_LOGF(GameplayMediaEncoder, Fatal, "Creating video encoder failed.");
		return false;
	}

	MemoryCheckpoint("Video encoder initialized");

	bIsOk = true; // So Shutdown is not called due to the ON_SCOPE_EXIT
	return true;
}

bool FGameplayMediaEncoder::Start()
{
	if(StartTime != 0)
	{
		UE_LOGF(GameplayMediaEncoder, Log, "Already running");
		return true;
	}

	if(!VideoEncoder)
	{
		UE_LOGF(GameplayMediaEncoder, Log, "Not initialized yet , so also performing a Intialize()");
		if(!Initialize())
		{
			return false;
		}
	}

	StartTime = FTimespan::FromSeconds(FPlatformTime::Seconds());
	AudioClock = 0;
	NumCapturedFrames = 0;

	//
	// subscribe to engine delegates for audio output and back buffer
	//

	FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
	if(AudioDevice)
	{
		bAudioFormatChecked = false;
		AudioDevice->RegisterSubmixBufferListener(AsShared(), AudioDevice->GetMainSubmixObject());
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().AddRaw(this, &FGameplayMediaEncoder::OnFrameBufferReady);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return true;
}

void FGameplayMediaEncoder::Stop()
{
	check(IsInGameThread());

	if(StartTime == 0)
	{
		UE_LOGF(GameplayMediaEncoder, Log, "Not running");
		return;
	}

	if(UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		if(FAudioDevice* AudioDevice = GameEngine->GetMainAudioDeviceRaw())
		{
			// AsShared() cannot be called in the destructor of the ref counted object!
			if(!bIsInDestructor)
			{
				AudioDevice->UnregisterSubmixBufferListener(AsShared(), AudioDevice->GetMainSubmixObject());
			}
		}

		if(FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().RemoveAll(this);
		}
	}

	StartTime = 0;
	AudioClock = 0;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
AVEncoder::FAudioConfig FGameplayMediaEncoder::GetAudioConfig() const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if(AudioEncoder)
	{
		auto codec = AudioEncoder->GetType();
		auto config = AudioEncoder->GetConfig();
		return {codec, config.Samplerate, config.NumChannels, config.Bitrate};
	}
	else
	{
		return {};
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FGameplayMediaEncoder::Shutdown()
{
	if(StartTime != 0)
	{
		UE_LOGF(GameplayMediaEncoder, Log, "Currently running, so also performing a Stop()");
		Stop();
	}

	{
		FScopeLock Lock(&AudioProcessingCS);
		if(AudioEncoder)
		{
			// AudioEncoder->Reset();
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			AudioEncoder->Shutdown();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			AudioEncoder.Reset();
		}
	}
	{
		FScopeLock Lock(&VideoProcessingCS);
		if(VideoEncoder)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			VideoEncoder->Shutdown();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			VideoEncoder.Reset();

			BackBuffers.Empty();
		}
	}
}

FTimespan FGameplayMediaEncoder::GetMediaTimestamp() const { return FTimespan::FromSeconds(FPlatformTime::Seconds()) - StartTime; }

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FString& FGameplayMediaEncoder::GetListenerName() const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	static const FString& ListenerName = TEXT("GameplayMediaEncoderListener");
	return ListenerName;
}
void FGameplayMediaEncoder::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double /*AudioClock*/)
{
	CSV_SCOPED_TIMING_STAT(GameplayMediaEncoder, OnNewSubmixBuffer);
	if(SampleRate != HardcodedAudioSamplerate)
	{
		// Only report the problem once
		if(!bAudioFormatChecked)
		{
			bAudioFormatChecked = true;
			UE_LOGF(GameplayMediaEncoder, Error, "Audio SampleRate needs to be %d HZ, current value is %d. VideoRecordingSystem won't record audio", HardcodedAudioSamplerate, SampleRate);
		}
		return;
	}

	ProcessAudioFrame(AudioData, NumSamples, NumChannels, SampleRate);
}

void FGameplayMediaEncoder::OnFrameBufferReady(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider)
{
	CSV_SCOPED_TIMING_STAT(GameplayMediaEncoder, OnBackBufferReady);
	check(IsInRenderingThread());
	ProcessVideoFrame(ViewportProvider.GetBackBufferResource());
}

void FGameplayMediaEncoder::FloatToPCM16(float const* floatSamples, int32 numSamples, TArray<int16>& out) const
{
	out.Reset(numSamples);
	out.AddZeroed(numSamples);

	float const* ptr = floatSamples;
	for(auto&& sample : out)
	{
		int32 N = *ptr >= 0 ? FMath::TruncToInt32(*ptr * int32(MAX_int16)) : FMath::TruncToInt32(*ptr * (int32(MAX_int16) + 1));
		sample = static_cast<int16>(FMath::Clamp(N, int32(MIN_int16), int32(MAX_int16)));
		ptr++;
	}
}

void FGameplayMediaEncoder::ProcessAudioFrame(const float* AudioData, int32 NumSamples, int32 NumChannels, int32 SampleRate)
{

	// Don't encode audio encoder is not setup or destroyed.
	if(!AudioEncoder.IsValid())
	{
		return;
	}

	//// convert to PCM data
	// TArray<int16> conversionBuffer;
	// FloatToPCM16(AudioData, NumSamples, conversionBuffer);

	//// add to any remainder data
	// PCM16.Append(conversionBuffer);

	//// encode the 10ms blocks
	// size_t const encodeBlockSize = AudioConfig.Samplerate / 100 * AudioConfig.NumChannels;
	// int32 bufferSize = PCM16.Num();
	// auto timestamp = GetMediaTimestamp().GetTicks();
	// auto bufferStart = PCM16.GetData();

	// while (bufferSize >= encodeBlockSize)
	//{
	//	rtc::Buffer encoded;
	//	auto encodedInfo = AudioEncoder->Encode(timestamp, { bufferStart, encodeBlockSize }, &encoded);
	//	if (encodedInfo.encoded_bytes > 0 || encodedInfo.send_even_if_empty)
	//		OnEncodedAudioFrame(encodedInfo, &encoded);
	//	timestamp += 10 * ETimespan::TicksPerMillisecond;
	//	bufferStart += encodeBlockSize;
	//	bufferSize -= encodeBlockSize;
	//}

	//// move the remainder to the start of the buffer
	// int32 const remainderIdx = bufferStart - PCM16.GetData();
	// for (int32 i = 0; i < bufferSize; ++i)
	//{
	//	PCM16[i] = PCM16[remainderIdx + i];
	//}
	// PCM16.SetNum(bufferSize, EAllowShrinking::No);

	Audio::AlignedFloatBuffer InData;
	InData.Append(AudioData, NumSamples);
	Audio::TSampleBuffer<float> FloatBuffer(InData, NumChannels, SampleRate);

	// Mix to stereo if required, since PixelStreaming only accept stereo at the moment
	if(FloatBuffer.GetNumChannels() != HardcodedAudioNumChannels)
	{
		FloatBuffer.MixBufferToChannels(HardcodedAudioNumChannels);
	}

	// Adjust the AudioClock if for some reason it falls behind real time. This can happen if the game spikes, or if we break into the debugger.
	FTimespan Now = GetMediaTimestamp();
	if(AudioClock < Now.GetTotalSeconds())
	{
		UE_LOGF(GameplayMediaEncoder, Warning, "Audio clock falling behind real time clock by %.3f seconds. Ajusting audio clock", Now.GetTotalSeconds() - AudioClock);
		// Put it slightly ahead of the real time clock
		AudioClock = Now.GetTotalSeconds() + (FloatBuffer.GetSampleDuration() / 2);
	}

	// Convert to signed PCM 16-bits
	// PCM16.Reset(FloatBuffer.GetNumSamples());
	// PCM16.AddZeroed(FloatBuffer.GetNumSamples());
	// int blockSize = AudioConfig.Samplerate / 100 * AudioConfig.NumChannels;
	// PCM16.Reset(blockSize);
	// PCM16.AddZeroed(blockSize);
	// const float* Ptr = reinterpret_cast<const float*>(FloatBuffer.GetData());
	// auto timestamp = GetMediaTimestamp().GetTotalMilliseconds();
	// for (int i = 0; i < NumSamples * NumChannels; ++i)
	//{
	//	int u;
	//	for (u = 0; u < blockSize && i < NumSamples; ++u, ++i, ++Ptr)
	//	{
	//		int32 N = *Ptr >= 0 ? *Ptr * int32(MAX_int16) : *Ptr * (int32(MAX_int16) + 1);
	//		PCM16[u] = static_cast<int16>(FMath::Clamp(N, int32(MIN_int16), int32(MAX_int16)));
	//	}

	//	rtc::Buffer encoded;
	//	auto encodedInfo = AudioEncoder->Encode(timestamp, { PCM16.GetData(), static_cast<size_t>(u) }, &encoded);
	//	OnEncodedAudioFrame(encodedInfo, &encoded);
	//	timestamp += 10;
	//}

	// for (int16& S : PCM16)
	//{
	//	int32 N = *Ptr >= 0 ? *Ptr * int32(MAX_int16) : *Ptr * (int32(MAX_int16) + 1);
	//	S = static_cast<int16>(FMath::Clamp(N, int32(MIN_int16), int32(MAX_int16)));
	//	Ptr++;
	//}

	////rtc::ArrayView<const int16_t> audio(PCM16.GetData(), FloatBuffer.GetNumSamples());
	// rtc::ArrayView<const int16_t> audio(PCM16.GetData(), blockSize);
	// auto testSampleRate = AudioEncoder->SampleRateHz();
	// auto testNumChannels = AudioEncoder->NumChannels();
	// check(testSampleRate == AudioConfig.Samplerate);
	// check(testNumChannels == AudioConfig.NumChannels);
	// auto timestamp = GetMediaTimestamp().GetTotalMilliseconds();
	// auto encodedInfo = AudioEncoder->Encode(timestamp, audio, &encoded);
	// OnEncodedAudioFrame(encodedInfo, &encoded);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AVEncoder::FAudioFrame Frame;
	Frame.Timestamp = FTimespan::FromSeconds(AudioClock);
	Frame.Duration = FTimespan::FromSeconds(FloatBuffer.GetSampleDuration());
	FloatBuffer.Clamp();
	Frame.Data = FloatBuffer;
	AudioEncoder->Encode(Frame);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	AudioClock += FloatBuffer.GetSampleDuration();
}

void FGameplayMediaEncoder::ProcessVideoFrame(const FTextureRHIRef& FrameBuffer)
{
	// Early exit is video encoder is not valid because it is not setup or has been destroyed
	if(!VideoEncoder.IsValid())
	{
		return;
	}

	FScopeLock Lock(&VideoProcessingCS);

	FTimespan Now = GetMediaTimestamp();

	if(bDoFrameSkipping)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		uint64 NumExpectedFrames = static_cast<uint64>(Now.GetTotalSeconds() * VideoConfig.Framerate);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		UE_LOG(GameplayMediaEncoder, VeryVerbose, TEXT("time %.3f: captured %" UINT64_FMT ", expected %" UINT64_FMT), Now.GetTotalSeconds(), NumCapturedFrames + 1, NumExpectedFrames);
		if(NumCapturedFrames + 1 > NumExpectedFrames)
		{
			UE_LOGF(GameplayMediaEncoder, Verbose, "Framerate control dropped captured frame");
			return;
		}
	}

	UpdateVideoConfig();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<AVEncoder::FVideoEncoderInputFrame> InputFrame = ObtainInputFrame();
	const int32 FrameId = InputFrame->GetFrameID();
	InputFrame->SetTimestampUs(Now.GetTicks());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CopyTexture(FrameBuffer, BackBuffers[InputFrame]);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AVEncoder::FVideoEncoder::FEncodeOptions EncodeOptions;
	VideoEncoder->Encode(InputFrame, EncodeOptions);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	LastVideoInputTimestamp = Now;
	NumCapturedFrames++;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedPtr<AVEncoder::FVideoEncoderInputFrame> FGameplayMediaEncoder::ObtainInputFrame()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<AVEncoder::FVideoEncoderInputFrame> InputFrame = VideoEncoderInput->ObtainInputFrame();
	InputFrame->SetWidth(VideoConfig.Width);
	InputFrame->SetHeight(VideoConfig.Height);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if(!BackBuffers.Contains(InputFrame))
	{
#if PLATFORM_WINDOWS && PLATFORM_DESKTOP
		const ERHIInterfaceType RHIType = RHIGetInterfaceType();

		const FRHITextureCreateDesc Desc =
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FRHITextureCreateDesc::Create2D(TEXT("VideoCapturerBackBuffer"), VideoConfig.Width, VideoConfig.Height, PF_B8G8R8A8)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			.SetFlags(ETextureCreateFlags::Shared | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV)
			.SetInitialState(ERHIAccess::CopyDest);

		if (RHIType == ERHIInterfaceType::D3D11)
		{
			FTextureRHIRef Texture = FRHICommandListImmediate::Get().CreateTexture(Desc);

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			InputFrame->SetTexture((ID3D11Texture2D*)Texture->GetNativeResource(), [&, InputFrame](ID3D11Texture2D* NativeTexture) { BackBuffers.Remove(InputFrame); });
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			BackBuffers.Add(InputFrame, Texture);
		}
		else if (RHIType == ERHIInterfaceType::D3D12)
		{
			FTextureRHIRef Texture = FRHICommandListImmediate::Get().CreateTexture(Desc);

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			InputFrame->SetTexture((ID3D12Resource*)Texture->GetNativeResource(), [&, InputFrame](ID3D12Resource* NativeTexture) { BackBuffers.Remove(InputFrame); });
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			BackBuffers.Add(InputFrame, Texture);
		}

		UE_LOGF(LogTemp, Log, "%d backbuffers currently allocated", BackBuffers.Num());
#else
		unimplemented();
#endif	// PLATFORM_DESKTOP && !PLATFORM_APPLE
	}

	return InputFrame;
}

void FGameplayMediaEncoder::SetVideoBitrate(uint32 Bitrate)
{
	NewVideoBitrate = Bitrate;
	bChangeBitrate = true;
}

void FGameplayMediaEncoder::SetVideoFramerate(uint32 Framerate)
{
	NewVideoFramerate = FMath::Clamp(Framerate, MinVideoFPS, MaxVideoFPS);
	bChangeFramerate = true;
}

void FGameplayMediaEncoder::UpdateVideoConfig()
{
	if(bChangeBitrate || bChangeFramerate)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		auto config = VideoEncoder->GetLayerConfig(0);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if(bChangeBitrate)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			config.MaxBitrate = MaxVideoBitrate;
			config.TargetBitrate = NewVideoBitrate;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		if(bChangeFramerate)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			config.MaxFramerate = NewVideoFramerate;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			NumCapturedFrames = 0;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		VideoEncoder->UpdateLayerConfig(0, config);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bChangeFramerate = false;
		bChangeBitrate = false;
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FGameplayMediaEncoder::OnEncodedAudioFrame(const AVEncoder::FMediaPacket& Packet)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{

	FScopeLock Lock(&ListenersCS);
	for(auto&& Listener : Listeners)
	{
		Listener->OnMediaSample(Packet);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FGameplayMediaEncoder::OnEncodedVideoFrame(uint32 LayerIndex, const TSharedPtr<AVEncoder::FVideoEncoderInputFrame> InputFrame, const AVEncoder::FCodecPacket& Packet)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AVEncoder::FMediaPacket packet(AVEncoder::EPacketType::Video);

	packet.Timestamp = InputFrame->GetTimestampUs();
	packet.Duration = 0; // This should probably be 1.0f / fps in ms
	packet.Data = TArray<uint8>(Packet.Data.Get(), Packet.DataSize);
	packet.Video.bKeyFrame = Packet.IsKeyFrame;
	packet.Video.Width = InputFrame->GetWidth();
	packet.Video.Height = InputFrame->GetHeight();
	packet.Video.FrameAvgQP = Packet.VideoQP;
	packet.Video.Framerate = VideoConfig.Framerate;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FScopeLock Lock(&ListenersCS);
	for(auto&& Listener : Listeners)
	{
		Listener->OnMediaSample(packet);
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InputFrame->Release();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


void FGameplayMediaEncoder::CopyTexture(const FTextureRHIRef& SourceTexture, FTextureRHIRef& DestinationTexture) const
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	if(SourceTexture->GetFormat() == DestinationTexture->GetFormat() && SourceTexture->GetSizeXY() == DestinationTexture->GetSizeXY())
	{
		TransitionAndCopyTexture(RHICmdList, SourceTexture, DestinationTexture, {});
	}
	else // Texture format mismatch, use a shader to do the copy.
	{
		IRendererModule* RendererModule = &FModuleManager::GetModuleChecked<IRendererModule>("Renderer");

		// #todo-renderpasses there's no explicit resolve here? Do we need one?
		FRHIRenderPassInfo RPInfo(DestinationTexture, ERenderTargetActions::Load_Store);

		RHICmdList.Transition(FRHITransitionInfo(DestinationTexture, ERHIAccess::Unknown, ERHIAccess::RTV));
		RHICmdList.BeginRenderPass(RPInfo, TEXT("CopyBackbuffer"));

		{
			RHICmdList.SetViewport(0, 0, 0.0f, (float)DestinationTexture->GetSizeX(), (float)DestinationTexture->GetSizeY(), 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			// New engine version...
			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
			TShaderMapRef<FScreenPS> PixelShader(ShaderMap);

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			const bool bSameSize = (DestinationTexture->GetDesc().Extent == SourceTexture->GetDesc().Extent);
			FRHISamplerState* PixelSampler = bSameSize ? TStaticSamplerState<SF_Point>::GetRHI() : TStaticSamplerState<SF_Bilinear>::GetRHI();

			SetShaderParametersLegacyPS(RHICmdList, PixelShader, PixelSampler, SourceTexture);

			RendererModule->DrawRectangle(RHICmdList, 0, 0,                // Dest X, Y
			                              (float)DestinationTexture->GetSizeX(),  // Dest Width
			                              (float)DestinationTexture->GetSizeY(),  // Dest Height
			                              0, 0,                            // Source U, V
			                              1, 1,                            // Source USize, VSize
			                              DestinationTexture->GetSizeXY(), // Target buffer size
			                              FIntPoint(1, 1),                 // Source texture size
			                              VertexShader, EDRF_Default);
		}

		RHICmdList.EndRenderPass();
		RHICmdList.Transition(FRHITransitionInfo(DestinationTexture, ERHIAccess::RTV, ERHIAccess::SRVMask));
	}
}
