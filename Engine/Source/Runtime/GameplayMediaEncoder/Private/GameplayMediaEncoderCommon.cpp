// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayMediaEncoderCommon.h"
#include "RHI.h"

TArray<FMemoryCheckpoint> gMemoryCheckpoints;
uint64 MemoryCheckpoint(const FString& Name)
{
#if PLATFORM_WINDOWS
	return 0;
#else
	uint64_t UsedPhysical = FPlatformMemory::GetMemoryUsedFast();

	static uint64 PeakMemory = 0;

	FMemoryCheckpoint Check;
	Check.Name = Name;
	static uint64_t FirstUsedPhysical = UsedPhysical;
	
	Check.UsedPhysicalMB = UsedPhysical / double(1024 * 1024);
	Check.DeltaMB = 0;
	Check.AccumulatedMB = (UsedPhysical - FirstUsedPhysical) / double(1024 * 1024);
	if (gMemoryCheckpoints.Num())
	{
		Check.DeltaMB = Check.UsedPhysicalMB - gMemoryCheckpoints.Last().UsedPhysicalMB;
	}
	gMemoryCheckpoints.Add(Check);
	return UsedPhysical;
#endif
}

void LogMemoryCheckpoints(const FString& Name)
{
	UE_LOGF(GameplayMediaEncoder, Log, "Memory breakdown: %ls...", *Name);
	for (const FMemoryCheckpoint& a : gMemoryCheckpoints)
	{
		UE_LOGF(GameplayMediaEncoder, Log, "%ls: UsedPhysicalMB=%4.3f, DeltaMB=%4.3f, AccumulatedMB=%4.3f",
			*a.Name, a.UsedPhysicalMB, a.DeltaMB, a.AccumulatedMB);
	}
}

