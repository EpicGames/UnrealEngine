// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Logging/LogMacros.h"
#include "RHIStaticStates.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/Thread.h"
#include "HAL/Event.h"
#include "Misc/ScopeExit.h"
#include "ShaderCore.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RHI.h"
#include "RHIResources.h"

//
// Macros to control some things during development
//
#define LIVESTREAMING 0

//
// Includes common to Windows and XboxOne
//
#if PLATFORM_WINDOWS
	#include "Templates/RefCounting.h"
#endif

#ifndef WMFMEDIA_SUPPORTED_PLATFORM
	#define WMFMEDIA_SUPPORTED_PLATFORM (PLATFORM_WINDOWS && (WINVER >= 0x0600 /*Vista*/) && !UE_SERVER)
#endif

DECLARE_LOG_CATEGORY_EXTERN(GameplayMediaEncoder, Log, VeryVerbose);

struct FMemoryCheckpoint
{
	FString Name;
	float UsedPhysicalMB;
	float DeltaMB;
	float AccumulatedMB;
};
extern TArray<FMemoryCheckpoint> gMemoryCheckpoints;
uint64 MemoryCheckpoint(const FString& Name);
void LogMemoryCheckpoints(const FString& Name);

