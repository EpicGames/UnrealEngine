// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "GameplayMediaEncoderCommon.h"
#include "GameplayMediaEncoder.h"

class FGameplayMediaEncoderModule : public IModuleInterface
{
public:
	FGameplayMediaEncoderModule()
	{
	}

	~FGameplayMediaEncoderModule()
	{
	}

	void StartupModule() override
	{
		FModuleManager::Get().LoadModule(TEXT("AVEncoder"));
	}

	void ShutdownModule() override
	{
		// If the FGameplayMediaEncoder instance was created, then explicitly destroy it here
		// instead of waiting for the automatic cleanup, since at that point some objects
		// it depends to for a clean shutdown are not available any longer.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FGameplayMediaEncoder::Singleton.Reset();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
private:
};

IMPLEMENT_MODULE(FGameplayMediaEncoderModule, GameplayMediaEncoder);

