// Copyright Epic Games, Inc. All Rights Reserved.

#include "IbmLiveStreaming.h"

#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"
#include "Serialization/JsonReader.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/ScopeLock.h"
#include "VideoRecordingSystem.h"
#include "Algo/FindSequence.h"

#if defined(WITH_IBMRTMPINGEST) && LIVESTREAMING

DEFINE_LOG_CATEGORY(IbmLiveStreaming);

extern "C"
{
	// rtmp_c_static.lib needs us to define this symbol
	char* build_number = "1";
}

// From IBM's sample
typedef enum _AACPacketType {
        AACSequenceHeader,
        AACRaw
} AACPacketType;

FAutoConsoleCommand LiveStreamingStart(
	TEXT("LiveStreaming.Start"),
	TEXT("Starts live streaming gameplay to IBM Cloud Video"),
	FConsoleCommandDelegate::CreateStatic(&FIbmLiveStreaming::StartCmd)
);

FAutoConsoleCommand LiveStreamingStop(
	TEXT("LiveStreaming.Stop"),
	TEXT("Stops live streaming"),
	FConsoleCommandDelegate::CreateStatic(&FIbmLiveStreaming::StopCmd)
);

static TAutoConsoleVariable<FString> CVarLiveStreamingClientId(
	TEXT("LiveStreaming.ClientId"),
	TEXT("76898b5ebb31c697023fccb47a3e7a7eb6aacb83"),
	TEXT("IBMCloudVideo ClientId to use for live streaming"));

static TAutoConsoleVariable<FString> CVarLiveStreamingClientSecret(
	TEXT("LiveStreaming.ClientSecret"),
	TEXT("11b3d99c37f85943224d710f037f2c5e8c8fe673"),
	TEXT("IBMCloudVideo ClientSecret to use for live streaming"));

static TAutoConsoleVariable<FString> CVarLiveStreamingChannel(
	TEXT("LiveStreaming.Channel"),
	TEXT("23619107"),
	TEXT("IBMCloudVideo Channel to use for live streaming"));

static TAutoConsoleVariable<float> CVarLiveStreamingMaxBitrate(
	TEXT("LiveStreaming.MaxBitrate"),
	30,
	TEXT("LiveStreaming: max allowed bitrate, in Mbps"));

static TAutoConsoleVariable<float> CVarLiveStreamingSpaceToEmptyQueue(
	TEXT("LiveStreaming.SpaceToEmptyQueue"),
	0.8,
	TEXT("LiveStreaming: factor to set bitrate a bit lower than available b/w to let IBMRTMP lib to empty the queue"));

static TAutoConsoleVariable<float> CVarLiveStreamingNonCongestedBitrateIncrease(
	TEXT("LiveStreaming.NonCongestedBitrateIncrease"),
	1.2,
	TEXT("LiveStreaming: rate of growth of non-congested bitrate"));

static TAutoConsoleVariable<float> CVarLiveStreamingBitrateThresholdToSwitchFPS(
	TEXT("LiveStreaming.BitrateThresholdToSwitchFPS"),
	15,
	TEXT("LiveStreaming: bitrate threshold to switch to lower FPS, in Mbps"));

static TAutoConsoleVariable<int32> CVarLiveStreamingDownscaledFPS(
	TEXT("LiveStreaming.DownscaledFPS"),
	30,
	TEXT("LiveStreaming: framerate to switch if poor uplink is detected"));

static TAutoConsoleVariable<float> CVarLiveStreamingBitrateProximityTolerance(
	TEXT("LiveStreaming.BitrateProximityTolerance"),
	1.2,
	TEXT("LiveStreaming: How close should the bitrate be to the reported bandwidth, to be recognized as fulfilling the available bandwidth"));

//
// FIbmLiveStreaming::FFormData
//

FIbmLiveStreaming::FFormData::FFormData()
{
	// To understand the need of the Boundary string, see https://www.w3.org/Protocols/rfc1341/7_2_Multipart.html
	BoundaryLabel = FString("e543322540af456f9a3773049ca02529-") + FString::FromInt(FMath::Rand());
	BoundaryBegin = FString("--") + BoundaryLabel + FString("\r\n");
	BoundaryEnd = FString("\r\n--") + BoundaryLabel + FString("--\r\n");
}

void FIbmLiveStreaming::FFormData::AddField(const FString& Name, const FString& Value)
{
	Data += FString("\r\n")
		+ BoundaryBegin
		+ FString("Content-Disposition: form-data; name=\"")
		+ Name
		+ FString("\"\r\n\r\n")
		+ Value;
}

// Convert  FString to UTF8 and put it in a TArray
TArray<uint8> FIbmLiveStreaming::FFormData::FStringToUint8(const FString& InString)
{
	TArray<uint8> OutBytes;

	// Handle empty strings
	if (InString.Len() > 0)
	{
		FTCHARToUTF8 Converted(*InString); // Convert to UTF8
		OutBytes.Append(reinterpret_cast<const uint8*>(Converted.Get()), Converted.Length());
	}

	return OutBytes;
}

TSharedRef<IHttpRequest> FIbmLiveStreaming::FFormData::CreateHttpPostRequestImpl(const FString& URL)
{
	TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();

	TArray<uint8> CombinedContent;
	CombinedContent.Append(FStringToUint8(Data));
	//CombinedContent.Append(MultiPartContent);
	CombinedContent.Append(FStringToUint8(BoundaryEnd));

	HttpRequest->SetHeader(TEXT("Content-Type"), FString("multipart/form-data; boundary=") + BoundaryLabel);
	HttpRequest->SetURL(URL);
	HttpRequest->SetVerb(TEXT("POST"));
	HttpRequest->SetContent(CombinedContent);

	return HttpRequest;
}

//
// FIbmLiveStreaming
//

FIbmLiveStreaming* FIbmLiveStreaming::Singleton = nullptr;

FIbmLiveStreaming::FIbmLiveStreaming()
{
	//
	// Create the RTMCData instance.
	// Ideally, we should be using rtmp_c_getinstance to query if the internal RTMPCData instance was set,
	// so thus initialize it or destroy it accordingly, but at the time of writing, rtmp_c_getinstance
	// logs an error if the internal instance is not set, so we avoid using rtmp_c_getinstance to keep
	// the logs clean.
	UE_LOGF(IbmLiveStreaming, Log, "Creating rtmpc object");
	RTMPCData* rtmp_c = rtmp_c_alloc(502, 1, 1, "Windows", "Windows device", TCHAR_TO_ANSI(*FGuid::NewGuid().ToString()), "en");
	check(rtmp_c);
	rtmp_c->appFlavor = AppFlavorUstream;
	rtmp_c_start(rtmp_c);
}

FIbmLiveStreaming::~FIbmLiveStreaming()
{
	Stop();

	RTMPCData* rtmp_c = rtmp_c_getinstance();
	check(rtmp_c);
	UE_LOGF(IbmLiveStreaming, Verbose, "Stopping rtmp_c instance");
	rtmp_c_stop(rtmp_c);
	UE_LOGF(IbmLiveStreaming, Verbose, "Freeing rtmp_c instance");
	rtmp_c_free(rtmp_c);
}

FIbmLiveStreaming* FIbmLiveStreaming::Get()
{
	if (!Singleton)
	{
		Singleton = new FIbmLiveStreaming();
	}

	return Singleton;
}

void FIbmLiveStreaming::CustomLogMsg(const char* Msg)
{
	FString FinalMsg(Msg);
	// IBM RTMP Ingest appends a \n to the end of the string. We need to remove it so we don't have empty lines in our logs
	FinalMsg.TrimEndInline();
	UE_LOGF(IbmLiveStreaming, Verbose, "IBMRTMPIngest: %ls", *FinalMsg);
}

bool FIbmLiveStreaming::CheckState(const wchar_t* FuncName, EState ExpectedState)
{
	if (State == ExpectedState)
	{
		return true;
	}
	else
	{
		UE_LOGF(IbmLiveStreaming, Error, "%ls: Wrong state (%d instead of %d)", FuncName, static_cast<int>(State.Load()), static_cast<int>(ExpectedState));
		return false;
	}
}

bool FIbmLiveStreaming::CheckState(const wchar_t* FuncName, EState ExpectedStateMin, EState ExpectedStateMax)
{
	if (State >= ExpectedStateMin && State<=ExpectedStateMax)
	{
		return true;
	}
	else
	{
		UE_LOGF(IbmLiveStreaming, Error, "%ls: Wrong state (%d instead of [%d..%d])", FuncName, static_cast<int>(State.Load()), static_cast<int>(ExpectedStateMin), static_cast<int>(ExpectedStateMax));
		return false;
	}
}

bool FIbmLiveStreaming::Start()
{
	if (!CheckState(__FUNCTIONW__, EState::None))
	{
		return false;
	}

	FGameplayMediaEncoder* MediaEncoder = FGameplayMediaEncoder::Get();
	// First thing we need to do is call RegisterListener, which initializes FGameplayMediaEncoder if this is the first listener
	MediaEncoder->RegisterListener(this);

	// Get SampleRate and NumChannels
	uint32 AudioSampleRate, AudioNumChannels, AudioBitrate;
	TRefCountPtr<IMFMediaType> AudioOutputType;
	verify(MediaEncoder->GetAudioOutputType(AudioOutputType));
	CHECK_HR(AudioOutputType->GetUINT32(MF_MT_AUDIO_SAMPLES_PER_SECOND, &AudioSampleRate));
	CHECK_HR(AudioOutputType->GetUINT32(MF_MT_AUDIO_NUM_CHANNELS, &AudioNumChannels));
	CHECK_HR(AudioOutputType->GetUINT32(MF_MT_AUDIO_AVG_BYTES_PER_SECOND, &AudioBitrate));

	if (CVarLiveStreamingClientId.GetValueOnAnyThread().IsEmpty() ||
		CVarLiveStreamingClientSecret.GetValueOnAnyThread().IsEmpty() ||
		CVarLiveStreamingChannel.GetValueOnAnyThread().IsEmpty())
	{
		UE_LOGF(IbmLiveStreaming, Error, "ClientId/ClientSecret/Channel not specified.");
		return false;
	}

	return Start(
		CVarLiveStreamingClientId.GetValueOnAnyThread(),
		CVarLiveStreamingClientSecret.GetValueOnAnyThread(),
		CVarLiveStreamingChannel.GetValueOnAnyThread(),
		AudioSampleRate, AudioNumChannels, AudioBitrate);
}

bool FIbmLiveStreaming::Start(const FString& ClientId, const FString& ClientSecret, const FString& Channel, uint32 AudioSampleRate, uint32 AudioNumChannels, uint32 AudioBitrate)
{

	MemoryCheckpoint("IbmLiveStreaming: Before start");

	check(IsInGameThread());

	if (!CheckState(__FUNCTIONW__, EState::None))
	{
		return false;
	}

	// NOTE: Repeated calls to RegisterListener(this) are ignored
	FGameplayMediaEncoder::Get()->RegisterListener(this);

	if (!((AudioSampleRate == 44100 || AudioSampleRate == 48000) && (AudioNumChannels == 2)))
	{
		UE_LOGF(IbmLiveStreaming, Error, "Unsupported audio settings");
		return false;
	}

	custom_c_log_msg = &FIbmLiveStreaming::CustomLogMsg;

	AudioPacketsSent = 0;
	VideoPacketsSent = 0;

	Config.ClientId = ClientId;
	Config.ClientSecret = ClientSecret;
	Config.Channel = Channel;
	Config.AudioSampleRate = AudioSampleRate;
	Config.AudioNumChannels = AudioNumChannels;
	Config.AudioBitrate = AudioBitrate;

	UE_LOGF(IbmLiveStreaming, Log,
		"Initializing with ClientId=%ls, ClientSecret=%ls, Channel=%ls, AudioSampleRate=%d, AudioNumChannels=%d",
		*ClientId, *ClientSecret, *Channel, AudioSampleRate, AudioNumChannels);

	FFormData FormData;
	FormData.AddField(TEXT("grant_type"), TEXT("client_credentials"));
	FormData.AddField(TEXT("client_id"), Config.ClientId);
	FormData.AddField(TEXT("client_secret"), Config.ClientSecret);
	FormData.AddField(TEXT("scope"), TEXT("broadcaster"));
	TSharedRef<IHttpRequest> HttpRequest = FormData.CreateHttpPostRequest("https://www.ustream.tv/oauth2/token", this, &FIbmLiveStreaming::OnProcessRequestComplete);

	if (!HttpRequest->ProcessRequest())
	{
		UE_LOGF(IbmLiveStreaming, Error, "Failed to initialize request to get access token");
		return false;
	}

	State = EState::GettingAccessToken;

	MemoryCheckpoint("IbmLiveStreaming: After start");

	return true;
}

bool FIbmLiveStreaming::GetJsonField(const TSharedPtr<FJsonObject>& JsonObject, const FString& FieldName, FString& DestStr)
{
	if (JsonObject->TryGetStringField(FieldName, DestStr))
	{
		return true;
	}
	else
	{
		UE_LOGF(IbmLiveStreaming, Error, "Json field '%ls' not found or not a string", *FieldName);
		return false;
	}
}

bool FIbmLiveStreaming::GetJsonField(const TSharedPtr<FJsonObject>& JsonObject, const FString& FieldName, const TSharedPtr<FJsonObject>*& DestObj)
{
	if (JsonObject->TryGetObjectField(FieldName, DestObj))
	{
		return true;
	}
	else
	{
		UE_LOGF(IbmLiveStreaming, Error, "Json field '%ls' not found or not an object", *FieldName);
		return false;
	}
}

void FIbmLiveStreaming::OnProcessRequestComplete(FHttpRequestPtr SourceHttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	FString Content = HttpResponse->GetContentAsString();
	EHttpResponseCodes::Type ResponseCode = (EHttpResponseCodes::Type) HttpResponse->GetResponseCode();
	UE_LOGF(IbmLiveStreaming, Verbose, "Ingest reply: Code %d, Content= %ls", (int)ResponseCode, *Content);
	if (!EHttpResponseCodes::IsOk(ResponseCode))
	{
		UE_LOGF(IbmLiveStreaming, Error, "Http Request failed.");
		State = EState::None;
		return;
	}

	TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(Content);
	TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
	if (!(FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid()))
	{
		UE_LOGF(IbmLiveStreaming, Error, "Failed to deserialize http request reply as json.");
		State = EState::None;
		return;
	}

	if (State == EState::GettingAccessToken)
	{
		FString AccessToken, TokenType;
		if (!GetJsonField(JsonObject, TEXT("access_token"), AccessToken) ||
			!GetJsonField(JsonObject, TEXT("token_type"), TokenType))
		{
			State = EState::None;
			return;
		}
		UE_LOGF(IbmLiveStreaming, Verbose, "Token: %ls, Type:%ls", *AccessToken, *TokenType);

		//
		// Fire up the next stage's http request
		//
		TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
		HttpRequest->OnProcessRequestComplete().BindRaw(this, &FIbmLiveStreaming::OnProcessRequestComplete);
		HttpRequest->SetURL(FString::Printf(TEXT("https://api.ustream.tv/channels/%s/ingest.json"), *Config.Channel));
		HttpRequest->SetVerb(TEXT("GET"));
		HttpRequest->SetHeader(TEXT("Authorization"), TokenType + " " + AccessToken);
		if (!HttpRequest->ProcessRequest())
		{
			UE_LOGF(IbmLiveStreaming, Error, "Failed to initialize IBMRTMP Ingest");
			State = EState::None;
			return;
		}

		State = EState::GettingIngestSettings;
	}
	else if (State == EState::GettingIngestSettings)
	{
		const TSharedPtr<FJsonObject>* IngestObject;
		FString RtmpUrl;
		if (!GetJsonField(JsonObject, TEXT("ingest"), IngestObject) ||
			!GetJsonField(*IngestObject, TEXT("rtmp_url"), RtmpUrl) ||
			!GetJsonField(*IngestObject, TEXT("streaming_key"), Ingest.StreamingKey))
		{
			State = EState::None;
			return;
		}
		UE_LOGF(IbmLiveStreaming, Verbose, "rtmp_url: %ls, streaming_key: %ls", *RtmpUrl, *Ingest.StreamingKey);

		TArray<FString> Tokens;
		RtmpUrl.ParseIntoArray(Tokens, TEXT("/"));
		if (Tokens.Num() != 4)
		{
			UE_LOGF(IbmLiveStreaming, Error, "Failed to initialize IBMRTMP Ingest. 'rtmp_url' field doesn't have the expected format.");
			State = EState::None;
			return;
		}

		Ingest.RtmpUrl.Host = Tokens[1];
		Ingest.RtmpUrl.Application = Tokens[2];
		Ingest.RtmpUrl.Channel = Tokens[3];
		Ingest.RtmpUrl.ApplicationName = Ingest.RtmpUrl.Application + TEXT("/") + Ingest.RtmpUrl.Channel;
		Ingest.RtmpUrl.StreamNamePrefix = FString::FromInt(FMath::Rand() % 9000 + 1000);
		Ingest.RtmpUrl.StreamName = TEXT("broadcaster/live") + Ingest.RtmpUrl.StreamNamePrefix;
		Connect();
	}
}

TAutoConsoleVariable<int32> CVarLiveStreamingBwMeasureWithEmptyQueue(
	TEXT("LiveStreaming.bwMeasureWithEmptyQueue"),
	1000,
	TEXT("LiveStreaming: interval of reporting measured bandwdith when send queue is empty, in msecs"));

TAutoConsoleVariable<int32> CVarLiveStreamingBwMeasureWithNotEmptyQueue(
	TEXT("LiveStreaming.bwMeasureWithNotEmptyQueue"),
	300,
	TEXT("LiveStreaming: interval of reporting measured bandwdith when send queue is not empty, in msecs"));

void FIbmLiveStreaming::Connect()
{
	if (!CheckState(__FUNCTIONW__, EState::GettingIngestSettings))
	{
		return ;
	}

	State = EState::Connecting;

	Ctx.Client = rtmpclient_alloc();
	check(Ctx.Client);
	Ctx.Client->server = strcpy_alloc(TCHAR_TO_ANSI(*Ingest.RtmpUrl.Host));

	//
	// Connect module
	//
	Ctx.ConnectModule = rtmpmodule_connect_create(Ctx.Client, TCHAR_TO_ANSI(*Ingest.RtmpUrl.ApplicationName));
	check(Ctx.ConnectModule);
    Ctx.ConnectModule->rpin = strcpy_alloc(rtmp_c_getinstance()->rpin);
    Ctx.ConnectModule->streamingKey = strcpy_alloc(TCHAR_TO_ANSI(*Ingest.StreamingKey));
    Ctx.ConnectModule->clientType = strcpy_alloc("broadcaster");
    Ctx.ConnectModule->hlsCompatible = 1;
	Ctx.ConnectModule->streamNamePrefix = strcpy_alloc(TCHAR_TO_ANSI(*Ingest.RtmpUrl.StreamNamePrefix));
    Ctx.ConnectModule->onConnectionSuccess = FIbmLiveStreaming::OnConnectionSuccess;
    Ctx.ConnectModule->onConnectionError = FIbmLiveStreaming::OnConnectionError;
    Ctx.ConnectModule->deviceType = RTMPDeviceTypeMobile;
    Ctx.ConnectModule->customData = this;
	// Ctx.ConnectModule->hasPermanentContentUrl = 0;

	//
    // Broadcaster module
	//
	Ctx.BroadcasterModule = rtmpmodule_broadcaster_create(Ctx.Client, TCHAR_TO_ANSI(*Ingest.RtmpUrl.StreamName));
	check(Ctx.BroadcasterModule);
    Ctx.BroadcasterModule->onStreamPublished = FIbmLiveStreaming::OnStreamPublished;
    Ctx.BroadcasterModule->onStreamError = FIbmLiveStreaming::OnStreamError;
    Ctx.BroadcasterModule->onStreamDeleted = FIbmLiveStreaming::OnStreamDeleted;
    Ctx.BroadcasterModule->onStopPublish = FIbmLiveStreaming::OnStopPublish;
    Ctx.BroadcasterModule->autoPublish = 0;
    Ctx.BroadcasterModule->customData = this;

	Ctx.BroadcasterModule->bwMeasureMinInterval = CVarLiveStreamingBwMeasureWithEmptyQueue.GetValueOnAnyThread();
	Ctx.BroadcasterModule->bwMeasureMaxInterval = CVarLiveStreamingBwMeasureWithNotEmptyQueue.GetValueOnAnyThread();
	Ctx.BroadcasterModule->onStreamBandwidthChanged = FIbmLiveStreaming::OnStreamBandwidthChanged;

	rtmpclient_start(Ctx.Client);
}

void FIbmLiveStreaming::Stop()
{

	if (State == EState::Stopping)
	{
		return;
	}

	check(IsInGameThread());
	UE_LOGF(IbmLiveStreaming, Log, "Stopping");

	if (!CheckState(__FUNCTIONW__, EState::GettingAccessToken, EState::Connected))
	{
		return;
	}

	State = EState::Stopping;
	FGameplayMediaEncoder::Get()->UnregisterListener(this);

	{
		FScopeLock Lock(&CtxMutex);
		if (Ctx.BroadcasterModule)
		{
			rtmpmodule_broadcaster_stop_publish(Ctx.BroadcasterModule);
		}
	}

}

void FIbmLiveStreaming::FinishStopOnGameThread()
{
	check(IsInGameThread());

	UE_LOGF(IbmLiveStreaming, Log, "Finalizing stop on game thread");

	FGameplayMediaEncoder::Get()->UnregisterListener(this);

	FScopeLock Lock(&CtxMutex);
	if (Ctx.Client)
	{
		rtmpclient_free(Ctx.Client);
		Ctx.Client = nullptr;
	}
	Ctx.BroadcasterModule = nullptr;
	Ctx.ConnectModule = nullptr;

	State = EState::None;
}

//
// Utility helper functions
//
namespace
{
	static constexpr uint8 NAL[4] = { 0, 0, 0, 1 };

	const TArrayView<uint8> MakeArrayViewFromRange(const uint8* Begin, const uint8* End)
	{
		return TArrayView<uint8>(const_cast<uint8*>(Begin), static_cast<int32>(End - Begin));
	};

	//
	// Helper functions to make it easier to write to RawData objects
	//
	void RawDataPush(RawData* Pkt, uint8 Val)
	{
		check(Pkt->offset < Pkt->length);
		Pkt->data[Pkt->offset++] = Val;
	}

	void RawDataPush(RawData* Pkt, const uint8* Begin, const uint8* End)
	{
		int Len = static_cast<int>(End - Begin);
		check((Pkt->offset + Len) <= Pkt->length);
		FMemory::Memcpy(Pkt->data + Pkt->offset, Begin, Len);
		Pkt->offset += Len;
	}

}

RawData* FIbmLiveStreaming::GetAvccHeader(const TArrayView<uint8>& DataView, const uint8** OutPpsEnd)
{
	// Begin/End To make it easier to use
	const uint8* DataBegin = DataView.GetData();
	const uint8* DataEnd = DataView.GetData() + DataView.Num();

	// encoded frame should begin with NALU start code
	check(FMemory::Memcmp(DataBegin, NAL, sizeof(NAL)) == 0);

	const uint8* SpsEnd = Algo::FindSequence(MakeArrayViewFromRange(DataBegin + sizeof(NAL), DataEnd), NAL);
	check(SpsEnd);
	TArrayView<uint8> SPS = MakeArrayViewFromRange(DataBegin + sizeof(NAL), SpsEnd);

	if (SPS[0] == 0x09)
	{
		// now it's not an SPS but AUD and so we need to skip it. happens with AMD AMF encoder
		const uint8* SpsBegin = SpsEnd + sizeof(NAL);
		SpsEnd = Algo::FindSequence(MakeArrayViewFromRange(SpsBegin, DataEnd), NAL);
		check(SpsEnd);
		SPS = MakeArrayViewFromRange(SpsBegin, SpsEnd);
		check(SPS[0] == 0x67); // SPS first byte: [forbidden_zero_bit:1 = 0, nal_ref_idc:2 = 3, nal_unit_type:5 = 7]
	}

	const uint8* PpsBegin = SpsEnd + sizeof(NAL);
	const uint8* PpsEnd = Algo::FindSequence(MakeArrayViewFromRange(PpsBegin, DataEnd), NAL);
	// encoded frame can contain just SPS/PPS
	if (PpsEnd == nullptr)
	{
		PpsEnd = DataEnd;
	}
	const TArrayView<uint8> PPS = MakeArrayViewFromRange(PpsBegin, PpsEnd);

	// To avoid reallocating, calculate the required final size (This is checked for correctness at the end)
	int32 FinalSize = 16 + SPS.Num() + PPS.Num();
	RawData* Pkt = rawdata_alloc(FinalSize);

	static constexpr uint8 Hdr[5] = { 0x17, 0, 0, 0, 0 };
	RawDataPush(Pkt, Hdr, Hdr + sizeof(Hdr));

	// http://neurocline.github.io/dev/2016/07/28/video-and-containers.html
	// http://aviadr1.blogspot.com/2010/05/h264-extradata-partially-explained-for.html
	RawDataPush(Pkt, 0x01); // AVCC version
	RawDataPush(Pkt, SPS[1]); // profile
	RawDataPush(Pkt, SPS[2]); // compatibility
	RawDataPush(Pkt, SPS[3]); // level
	RawDataPush(Pkt, 0xFC | 3); // reserved (6 bits), NALU length size - 1 (2 bits)
	RawDataPush(Pkt, 0xE0 | 1); // reserved (3 bits), num of SPSs (5 bits)

	const uint8* SPS_size = Bytes(static_cast<const uint16&>(SPS.Num())); // 2 bytes for length of SPS
	RawDataPush(Pkt, SPS_size[1]);
	RawDataPush(Pkt, SPS_size[0]);

	RawDataPush(Pkt, SPS.GetData(), SPS.GetData() + SPS.Num());

	RawDataPush(Pkt, 1); // num of PPSs
	const uint8* PPS_size = Bytes(static_cast<const uint16&>(PPS.Num())); // 2 bytes for length of PPS
	RawDataPush(Pkt, PPS_size[1]);
	RawDataPush(Pkt, PPS_size[0]);
	RawDataPush(Pkt, PPS.GetData(), PPS.GetData() + PPS.Num()); // PPS data

	// Check if we calculated the required size exactly
	check(Pkt->offset == Pkt->length);

	*OutPpsEnd = PpsEnd;
	Pkt->offset = 0;
	return Pkt;
}

RawData* FIbmLiveStreaming::GetVideoPacket(const TArrayView<uint8>& DataView, bool bVideoKeyframe, const uint8* DataBegin)
{
	check(FMemory::Memcmp(DataBegin, NAL, sizeof(NAL)) == 0);

	// To make it easier to use
	const uint8* DataEnd = DataView.GetData() + DataView.Num();

	// To avoid reallocating, calculate the required final size (This is checked for correctness at the end)
	int FinalSize = 5 + 4 + (DataEnd - (DataBegin + sizeof(NAL)));
	RawData * Pkt = rawdata_alloc(FinalSize);

	RawDataPush(Pkt, bVideoKeyframe ? 0x17 : 0x27);
	RawDataPush(Pkt, 0x1);
	RawDataPush(Pkt, 0x0);
	RawDataPush(Pkt, 0x0);
	RawDataPush(Pkt, 0x0);

	int32 DataSizeValue = static_cast<int32>(DataEnd - DataBegin - sizeof(NAL));;
	verify(DataSizeValue > 0);
	const uint8* DataSize = Bytes(DataSizeValue);
	RawDataPush(Pkt, DataSize[3]);
	RawDataPush(Pkt, DataSize[2]);
	RawDataPush(Pkt, DataSize[1]);
	RawDataPush(Pkt, DataSize[0]);
	RawDataPush(Pkt, DataBegin + sizeof(NAL), DataEnd);

	// Check if we calculated the required size exactly
	check(Pkt->offset == Pkt->length);

	Pkt->offset = 0;
	return Pkt;
}

void FIbmLiveStreaming::QueueFrame(RTMPContentType FrameType, RawData* Pkt, uint32 TimestampMs)
{
	FScopeLock Lock(&CtxMutex);
	rtmpmodule_broadcaster_queue_frame(Ctx.BroadcasterModule, FrameType, Pkt, TimestampMs);
}

DECLARE_CYCLE_STAT(TEXT("IBMRTMP_Inject"), STAT_FIbmLiveStreaming_Inject, STATGROUP_VideoRecordingSystem);
DECLARE_CYCLE_STAT(TEXT("IBMRTMP_InjectVideo"), STAT_FIbmLiveStreaming_InjectVideo, STATGROUP_VideoRecordingSystem);
DECLARE_CYCLE_STAT(TEXT("IBMRTMP_InjectAudio"), STAT_FIbmLiveStreaming_InjectAudio, STATGROUP_VideoRecordingSystem);
void FIbmLiveStreaming::InjectVideo(uint32 TimestampMs, const TArrayView<uint8>& DataView, bool bIsKeyFrame)
{
	SCOPE_CYCLE_COUNTER(STAT_FIbmLiveStreaming_InjectVideo);

	UE_LOGF(IbmLiveStreaming, Verbose, "Injecting Video. TimestampMs=%u, Size=%u, bIsKeyFrame=%ls", TimestampMs, DataView.Num(), bIsKeyFrame ? TEXT("true"):TEXT("false"));

	//
	// From IBM's sample...
	//
	//video packet:
	//#1 config: 0x17,0,0,0,0,avcc | ts: 0
	//..n data: key 0x17 | inter: 0x27, 0x01, 0, 0, 0, data size without nal, video data after nal
	// sample_add_frame(ctx, data, data_lenght, timestamp, RTMPVideoDataPacketType);
	if (VideoPacketsSent == 0)
	{
		check(bIsKeyFrame); // the first packet always should be key-frame

		const uint8* PpsEnd;
		RawData* Pkt = GetAvccHeader(DataView, &PpsEnd);
		QueueFrame(RTMPVideoDataPacketType, Pkt, 0);

		if (PpsEnd != DataView.GetData() + DataView.Num()) // do we have any other NALUs in this frame?
		{
			Pkt = GetVideoPacket(DataView, bIsKeyFrame, PpsEnd);
			QueueFrame(RTMPVideoDataPacketType, Pkt, TimestampMs);
		}
	}
	else
	{
		RawData* Pkt = GetVideoPacket(DataView, bIsKeyFrame, DataView.GetData());
		QueueFrame(RTMPVideoDataPacketType, Pkt, TimestampMs);
	}

	++VideoPacketsSent;

	// Every N seconds, we log the average framerate
	{
		double Now = FPlatformTime::Seconds();
		double Elapsed = Now - FpsCalculationStartTime;
		if (Elapsed >= 4.0f)
		{
			UE_LOGF(IbmLiveStreaming, Verbose, "LiveStreaming Framerate = %3.2f",
				(VideoPacketsSent - FpsCalculationStartVideoPackets) / Elapsed);
			FpsCalculationStartVideoPackets = VideoPacketsSent;
			FpsCalculationStartTime = Now;
		}
	}

}

void FIbmLiveStreaming::InjectAudio(uint32 TimestampMs, const TArrayView<uint8>& DataView)
{
	SCOPE_CYCLE_COUNTER(STAT_FIbmLiveStreaming_InjectAudio);

	UE_LOGF(IbmLiveStreaming, Verbose, "Injecting Audio. TimestampMs=%u, Size=%u", TimestampMs, DataView.Num());

	//
	// From IBM's sample
	//
	//audio packet:
	 //#1 config: aac config (parser: aac_parse_extradata) 1byte (FLVCodecAAC|FLVAudioSampleRate(select rate)|FLVAudioSize(mostly 16b)|FLVAudioChannels(mostly stereo)), AACSequenceHeader, aac_lc_write_extradata() output | ts: 0
	//..n data: 1byte aac config, AACRaw, audio data
	// sample_add_frame(ctx, data, data_lenght, timestamp, RTMPAudioDataPacketType);

	// NOTE: FLVAudioSampleRate only goes up to 44kHz (FLVAudioSampleRate44kHz), but audio works fine at 48hkz too
	// because aac_lc_write_extradata allows specifying the correct sample rate
	constexpr unsigned char ConfigByte = FLVCodecAAC | FLVAudioSampleRate44kHz | FLVAudio16bit | FLVAudioStereo;
	if (!AudioPacketsSent == 0)
	{
		RawData* Pkt= rawdata_alloc(2);
		Pkt->data[0] = ConfigByte;
		Pkt->data[1] = AACSequenceHeader;
		Pkt->offset = 2;
		aac_lc_write_extradata(Pkt, Config.AudioSampleRate, Config.AudioNumChannels); // This adds another 2 bytes of configuration data 
		Pkt->offset = 0;
		QueueFrame(RTMPAudioDataPacketType, Pkt, 0);
	}

	RawData* Pkt= rawdata_alloc(2 + DataView.Num());
	Pkt->data[0] = ConfigByte;
	Pkt->data[1] = AACRaw;
	FMemory::Memcpy(Pkt->data + 2, DataView.GetData(), DataView.Num());
	QueueFrame(RTMPAudioDataPacketType, Pkt, TimestampMs);

	++AudioPacketsSent;
}

void FIbmLiveStreaming::OnMediaSample(const FGameplayMediaEncoderSample& Sample)
{
	SCOPE_CYCLE_COUNTER(STAT_FIbmLiveStreaming_Inject);
	FScopeLock Lock(&CtxMutex);

	if (State != EState::Connected)
	{
		return;
	}

	if (VideoPacketsSent == 0 && AudioPacketsSent == 0)
	{
		// We only start injecting when we receive a keyframe
		if (!Sample.IsVideoKeyFrame())
		{
			return;
		}

		LiveStreamStartTimespan = Sample.GetTime();
		FpsCalculationStartTime = LiveStreamStartTimespan.GetTotalSeconds();
	}

	uint32 TimestampMs = static_cast<uint32>((Sample.GetTime() - LiveStreamStartTimespan).GetTotalMilliseconds());

	TRefCountPtr<IMFMediaBuffer> MediaBuffer = nullptr;
	verify(SUCCEEDED(const_cast<IMFSample*>(Sample.GetSample())->GetBufferByIndex(0, MediaBuffer.GetInitReference())));
	BYTE* SrcData = nullptr;
	DWORD MaxLength = 0;
	DWORD CurrentLength = 0;
	verify(SUCCEEDED(MediaBuffer->Lock(&SrcData, &MaxLength, &CurrentLength)));
	// Append the payload to Data (in case it has any custom data there already)
	TArrayView<uint8> DataView(SrcData, CurrentLength);

	if (Sample.GetType() == EMediaType::Video)
	{
		InjectVideo(TimestampMs, DataView, Sample.IsVideoKeyFrame());
	}
	else if (Sample.GetType() == EMediaType::Audio)
	{
		InjectAudio(TimestampMs, DataView);
	}

	verify(SUCCEEDED(MediaBuffer->Unlock()));

	// #RVF : remove this once the code is production ready
	static bool FirstPacket = true;
	if (FirstPacket)
	{
		FirstPacket = false;
		MemoryCheckpoint("IbmLiveStreaming: After sending first packet");
		LogMemoryCheckpoints("After live streaming the first packet");
	}
}

void FIbmLiveStreaming::StopFromSocketThread()
{
	{
		FScopeLock Lock(&CtxMutex);
		rtmpclient_stop(Ctx.Client);
	}

	// Execute the Finalize event on the GameThread
	FGraphEventRef FinalizeEvent = FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateRaw(this, &FIbmLiveStreaming::FinishStopOnGameThread),
		TStatId(), nullptr, ENamedThreads::GameThread);
}

void FIbmLiveStreaming::OnConnectionErrorImpl(RTMPModuleConnect* Module, RTMPEvent Evt, void* RejectInfoObj)
{
	UE_LOGF(IbmLiveStreaming, Error, "%ls: Connect failed. Reason: %d", __FUNCTIONW__, Module->rejectReason);
	StopFromSocketThread();
}

void FIbmLiveStreaming::OnConnectionSuccessImpl(RTMPModuleConnect* Module)
{
	UE_LOGF(IbmLiveStreaming, Log, "%ls", __FUNCTIONW__);
	check(State == EState::Connecting);

	{
		FScopeLock Lock(&CtxMutex);
		rtmpmodule_broadcaster_start(Ctx.BroadcasterModule);
	}

	// Changing the state AFTER making the required IBM calls
	State = EState::Connected;
}

void FIbmLiveStreaming::OnStreamPublishedImpl(RTMPModuleBroadcaster* Module)
{
	UE_LOGF(IbmLiveStreaming, Log, "%ls", __FUNCTIONW__);
	check(State == EState::Connected);

	{
		FScopeLock Lock(&CtxMutex);
		rtmpmodule_broadcaster_start_publish(Ctx.BroadcasterModule);
	}

}

void FIbmLiveStreaming::OnStreamDeletedImpl(RTMPModuleBroadcaster* Module)
{
	UE_LOGF(IbmLiveStreaming, Log, "%ls", __FUNCTIONW__);
	StopFromSocketThread();
}

void FIbmLiveStreaming::OnStreamErrorImpl(RTMPModuleBroadcaster* Module)
{
	UE_LOGF(IbmLiveStreaming, Log, "%ls", __FUNCTIONW__);
	StopFromSocketThread();
}

void FIbmLiveStreaming::OnStopPublishImpl(RTMPModuleBroadcaster* Module)
{
	UE_LOGF(IbmLiveStreaming, Log, "%ls", __FUNCTIONW__);

	{
		FScopeLock Lock(&CtxMutex);
		rtmpmodule_broadcaster_stop(Ctx.BroadcasterModule);
	}
}

void FIbmLiveStreaming::OnStreamBandwidthChangedImpl(RTMPModuleBroadcaster* Module, uint32 Bandwidth, bool bQueueWasEmpty)
{
	UE_LOGF(IbmLiveStreaming, Verbose, "%ls", __FUNCTIONW__);
	check(State == EState::Connected);

	// if debugger is stopped on a breakpoint and then resumed, measured b/w will be very small. just ignore it
	if (Bandwidth <= Config.AudioBitrate)
	{
		UE_LOGF(IbmLiveStreaming, Warning, "low traffic detected (%.3f Mbps), have debugger been stopped on breakpoint?", Bandwidth / 10000000.0);
		return;
	}

	uint32 VideoBandwidth = Bandwidth - Config.AudioBitrate;

	//
	// Get current video framerate and framerate
	//
	TRefCountPtr<IMFMediaType> VideoOutputType;
	uint32 CurrentBitrate, CurrentFramerate, CurrentFramerateDenominator;
	verify(FGameplayMediaEncoder::Get()->GetVideoOutputType(VideoOutputType));
	verify(SUCCEEDED(MFGetAttributeRatio(VideoOutputType, MF_MT_FRAME_RATE, &CurrentFramerate, &CurrentFramerateDenominator)));
	verify(SUCCEEDED(VideoOutputType->GetUINT32(MF_MT_AVG_BITRATE, &CurrentBitrate)));

	// NOTE:
	// Reported bandwidth doesn't always mean available bandwidth, e.g. when we don't push enough data.
	// In general, `VideoBandwidth` value is either:
	// * encoder's output bitrate if available bandwidth is more than required for encoder with current settings
	// * currently available bandwidth if encoder's output bitrate is higher than that

	// NOTE:
	// Bitrate depends on framerate as it's in bits per second(!): e.g. to calculate how much bits per encoded
	// frame is allowed encoder divides configured bitrate by framerate.

	// bitrate control

	const double BitrateProximityTolerance = CVarLiveStreamingBitrateProximityTolerance.GetValueOnAnyThread();

	uint32 NewVideoBitrate = 0;
	uint32 NewVideoFramerate = 0;

	// - if queue is not empty we restrict bitrate to reported bandwidth;
	// - if queue is empty and reported bandwidth is close to current bitrate (encoder output is restricted
	// but potentially greater bandwidth is available) we increase bitrate by a constant factor.
	// we don't increase bitrate if encoder wasn't fulfilling available bandwidth
	if (!bQueueWasEmpty)
	{
		NewVideoBitrate = static_cast<uint32>(VideoBandwidth * CVarLiveStreamingSpaceToEmptyQueue.GetValueOnAnyThread());
	}
	else if (CurrentBitrate < VideoBandwidth * BitrateProximityTolerance)
	{
		// `VideoBandwidth` is often just a fluctuations of current encoder bitrate, so if it's too close to currently
		// configured bitrate let's be optimistic and try to stress network a little bit more,
		// mainly to increase bitrate recovery speed
		NewVideoBitrate = FMath::Max(static_cast<uint32>(CurrentBitrate * CVarLiveStreamingNonCongestedBitrateIncrease.GetValueOnAnyThread()), VideoBandwidth);
	}

	// framerate control
	// by default we stream 60FPS or as much as the game manages
	// if available b/w is lower than configurable threshold (e.g. 15Mbps) we downscale to e.g. 30FPS
	// (configurable) and restore back to 60FPS if b/w improves

	const uint32 BitrateThresholdToSwitchFPS = static_cast<uint32>(CVarLiveStreamingBitrateThresholdToSwitchFPS.GetValueOnAnyThread() * 1000 * 1000);
	// two values to avoid switching back and forth on bitrate fluctuations
	const uint32 BitrateThresholdDown = static_cast<uint32>(BitrateThresholdToSwitchFPS * BitrateProximityTolerance); // switch down to 30FPS if bitrate is lower than that
	const uint32 BitrateThresholdUp = static_cast<uint32>(BitrateThresholdToSwitchFPS / BitrateProximityTolerance); // switch up to 60FPS if bitrate is higher than that

	const uint32 DownscaledFPS = static_cast<uint32>(CVarLiveStreamingDownscaledFPS.GetValueOnAnyThread());

	if (NewVideoBitrate != 0)
	{
		if (CurrentFramerate > DownscaledFPS && NewVideoBitrate < BitrateThresholdDown &&
			NewVideoBitrate < CurrentBitrate) // don't downscale if bitrate is growing
		{
			NewVideoFramerate = DownscaledFPS;
		}
		else if (CurrentFramerate < HardcodedVideoFPS && NewVideoBitrate > BitrateThresholdUp)
		{
			NewVideoFramerate = HardcodedVideoFPS;
		}
	}

	// a special case:
	// if we are on downscaled framerate and encoder doesn't produce enough data to probe higher b/w
	// (happens when currently set bitrate is more than enough for encoder on downscaled FPS and it's lower
	// than threshold to switch to higher framerate), we won't switch to higher framerate even on unlimited b/w.
	// so whenever we detect that encoder doesn't push hard enough on downscaled framerate try to switch to
	// higher framerate and let the logic above kick in if the assumption is wrong
	//if (!bChangeBitrate && !bChangeFramerate && bQueueWasEmpty &&
	if (NewVideoBitrate == 0 && NewVideoFramerate == 0 && bQueueWasEmpty &&
		CurrentFramerate < HardcodedVideoFPS &&
		CurrentBitrate > VideoBandwidth * BitrateProximityTolerance)
	{
		NewVideoFramerate = HardcodedVideoFPS;
		UE_LOGF(IbmLiveStreaming, Verbose, "framerate control special case: switching to %dFPS", NewVideoFramerate);
	}

	if (NewVideoBitrate)
	{
		FGameplayMediaEncoder::Get()->SetVideoBitrate(NewVideoBitrate);
	}

	if (NewVideoFramerate)
	{
		FGameplayMediaEncoder::Get()->SetVideoFramerate(NewVideoFramerate);
	}

	UE_LOGF(IbmLiveStreaming, Verbose,
		"reported b/w %.3f Mbps, video b/w %.3f, queue empty %d, configured bitrate %.3f, new bitrate %.3f, configured %dFPS, new %dFPS",
		Bandwidth / 1000000.0, VideoBandwidth / 1000000.0, bQueueWasEmpty, CurrentBitrate / 1000000.0,
		NewVideoBitrate ? NewVideoBitrate / 1000000.0 : -1.0,
		CurrentFramerate, NewVideoFramerate ? NewVideoFramerate : -1);
}

void FIbmLiveStreaming::OnConnectionError(RTMPModuleConnect* Module, RTMPEvent Evt, void* RejectInfoObj)
{
	reinterpret_cast<FIbmLiveStreaming*>(Module->customData)->OnConnectionErrorImpl(Module, Evt, RejectInfoObj);
}
void FIbmLiveStreaming::OnConnectionSuccess(RTMPModuleConnect* Module)
{
	reinterpret_cast<FIbmLiveStreaming*>(Module->customData)->OnConnectionSuccessImpl(Module);
}

void FIbmLiveStreaming::OnStreamPublished(RTMPModuleBroadcaster* Module)
{
	reinterpret_cast<FIbmLiveStreaming*>(Module->customData)->OnStreamPublishedImpl(Module);
}

void FIbmLiveStreaming::OnStreamDeleted(RTMPModuleBroadcaster* Module)
{
	reinterpret_cast<FIbmLiveStreaming*>(Module->customData)->OnStreamDeletedImpl(Module);
}

void FIbmLiveStreaming::OnStreamError(RTMPModuleBroadcaster* Module)
{
	reinterpret_cast<FIbmLiveStreaming*>(Module->customData)->OnStreamErrorImpl(Module);
}

void FIbmLiveStreaming::OnStopPublish(RTMPModuleBroadcaster* Module)
{
	reinterpret_cast<FIbmLiveStreaming*>(Module->customData)->OnStopPublishImpl(Module);
}

void FIbmLiveStreaming::OnStreamBandwidthChanged(RTMPModuleBroadcaster* Module, unsigned long Bandwidth, int32 QueueWasEmpty)
{
	reinterpret_cast<FIbmLiveStreaming*>(Module->customData)->OnStreamBandwidthChangedImpl(Module, Bandwidth, QueueWasEmpty != 0 );
}

#endif // WITH_IBMRTMPINGEST

