// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayMediaEncoder.h"
#include "Interfaces/IHttpRequest.h"
#include "GameplayMediaEncoderCommon.h"

#if defined(WITH_IBMRTMPINGEST) && LIVESTREAMING

// IBM RTMP Ingest
THIRD_PARTY_INCLUDES_START
	extern "C" {
		#include "rtmp_c/core/init.h"
		#include "rtmp_c/net/rtmp/rtmpclient.h"
		#include "rtmp_c/net/rtmp/modules/rtmpmodule_connect.h"
		#include "rtmp_c/net/rtmp/modules/rtmpmodule_broadcaster.h"
		#include "rtmp_c/core/rawdata.h"
		#include "rtmp_c/core/platform/threadfunc.h"
		#include "rtmp_c/codec/codecs.h"
	}
THIRD_PARTY_INCLUDES_END

class FJsonObject;

DECLARE_LOG_CATEGORY_EXTERN(IbmLiveStreaming, Log, All);

class FIbmLiveStreaming final : private IGameplayMediaEncoderListener
{
public:
	using FBandwidthProbeCallback = TFunction<void(uint32, bool)>;

	FIbmLiveStreaming();
	~FIbmLiveStreaming();

	static FIbmLiveStreaming* Get();

	/**
	* Starts streaming, using best guess settings
	*/
	bool Start();
	/**
	* Starts streaming, using explicit settings
	*/
	bool Start(const FString& ClientId, const FString& ClientSecret, const FString& Channel, uint32 AudioSampleRate, uint32 AudioNumChannels, uint32 AudioBitrate);
	/**
	* Stops streaming
	*/
	void Stop();

	static void StartCmd()
	{
		// We call Get(), so it creates the singleton
		Get()->Start();
	}

	static void StopCmd()
	{
		if (Singleton)
		{
			Singleton->Stop();
		}
	}

private:
	enum class EState
	{
		None,
		GettingAccessToken, // See https://developers.video.ibm.com/channel-api/getting-started.html#token-endpoint_8
		GettingIngestSettings, // See https://developers.video.ibm.com/channel-api/channel.html#ingest-settings_77
		Connecting,
		Connected,
		Stopping
	};

	struct FRtmpUrl
	{
		FString Host;
		FString Application;
		FString Channel;
		FString ApplicationName;
		FString StreamName;
		FString StreamNamePrefix;
	};

	struct FIngest
	{
		FString StreamingKey;
		FRtmpUrl RtmpUrl;
	};

	struct FFormData
	{
	public:
		FFormData();
		void AddField(const FString& Name, const FString& Value);

		// Call this in the end, to generate the HttpRequest
		template<typename T1, typename T2>
		TSharedRef<IHttpRequest> CreateHttpPostRequest(const FString& URL, T1&& TargetObj, T2&& TargetObjHandler)
		{
			auto HttpRequest = CreateHttpPostRequestImpl(URL);
			HttpRequest->OnProcessRequestComplete().BindRaw(std::forward<T1>(TargetObj), std::forward<T2>(TargetObjHandler));
			return HttpRequest;
		}

	private:
		static TArray<uint8> FStringToUint8(const FString& InString);

		FString BoundaryLabel;
		FString BoundaryBegin;
		FString BoundaryEnd;
		FString Data;

		TSharedRef<IHttpRequest> CreateHttpPostRequestImpl(const FString& URL);
	};

	// IGameplayMediaEncoderListener interface
	void OnMediaSample(const FGameplayMediaEncoderSample& Sample) override;

	/**
	 * Callback from HTTP library when a request has completed
	 * @param HttpRequest The request object
	 * @param HttpResponse The response from the server
	 * @param bSucceeded Whether a response was successfully received
	 */
	void OnProcessRequestComplete(FHttpRequestPtr SourceHttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);

	static bool GetJsonField(const TSharedPtr<FJsonObject>& JsonObject, const FString& FieldName, FString& DestStr);
	static bool GetJsonField(const TSharedPtr<FJsonObject>& JsonObject, const FString& FieldName, const TSharedPtr<FJsonObject>*& DestObj);

	void Connect();

	// Called to finalize a previous call to `Stop()`.  This is called once
	// the IBM library tells us the stream stopped and was deleted
	void FinishStopOnGameThread();
	void StopFromSocketThread();

	//
	// Callbacks for IBM RTMP C Ingest
	//
	static void OnConnectionError(RTMPModuleConnect* Module, RTMPEvent Evt, void* RejectInfoObj);
	static void OnConnectionSuccess(RTMPModuleConnect* Module);
	static void OnStreamPublished(RTMPModuleBroadcaster* Module);
	static void OnStreamDeleted(RTMPModuleBroadcaster* Module);
	static void OnStreamError(RTMPModuleBroadcaster* Module);
	static void OnStopPublish(RTMPModuleBroadcaster* Module);
	static void OnStreamBandwidthChanged(RTMPModuleBroadcaster* Module, unsigned long Bandwidth, int32 QueueWasEmpty);
	void OnConnectionErrorImpl(RTMPModuleConnect* Module, RTMPEvent Evt, void* RejectInfoObj);
	void OnConnectionSuccessImpl(RTMPModuleConnect* Module);
	void OnStreamPublishedImpl(RTMPModuleBroadcaster* Module);
	void OnStreamDeletedImpl(RTMPModuleBroadcaster* Module);
	void OnStreamErrorImpl(RTMPModuleBroadcaster* Module);
	void OnStopPublishImpl(RTMPModuleBroadcaster* Module);
	void OnStreamBandwidthChangedImpl(RTMPModuleBroadcaster* Module, uint32 Bandwidth, bool bQueueWasEmpty);

	static RawData* GetAvccHeader(const TArrayView<uint8>& DataView, const uint8** OutPpsEnd);
	static RawData* GetVideoPacket(const TArrayView<uint8>& Data, bool bVideoKeyframe, const uint8* DataBegin);

	void InjectVideo(uint32 TimestampMs, const TArrayView<uint8>& DataView, bool bIsKeyFrame);
	void InjectAudio(uint32 TimestampMs, const TArrayView<uint8>& DataView);

	template<typename T>
	static const uint8* Bytes(T& t)
	{
		return reinterpret_cast<const uint8*>(&t);
	}

	// Custom log function for the IBM librayr
	static void CustomLogMsg(const char* Msg);

	// Utility functions to log an error if we are in the wrong state
	bool CheckState(const wchar_t* FuncName, EState ExpectedState);
	bool CheckState(const wchar_t* FuncName, EState ExpectedStateMin, EState ExpectedStateMax);

	static FIbmLiveStreaming* Singleton;

	// Initialization parameters
	struct 
	{
		FString ClientId;
		FString ClientSecret;
		FString Channel;
		uint32 AudioSampleRate;
		uint32 AudioNumChannels;
		uint32 AudioBitrate;
	} Config;

	FIngest Ingest;

	TAtomic<EState> State = EState::None;
	TAtomic<uint32> AudioPacketsSent = 0;
	TAtomic<uint32> VideoPacketsSent = 0;
	FTimespan LiveStreamStartTimespan;
	uint32 FpsCalculationStartVideoPackets = 0;
	double FpsCalculationStartTime = 0;

	struct 
	{
		RTMPClient* Client = nullptr;
		RTMPModuleBroadcaster* BroadcasterModule = nullptr;
		RTMPModuleConnect* ConnectModule = nullptr;
	} Ctx;
	FCriticalSection CtxMutex;

	// Helper function that locks Ctx and make the IBM call
	void QueueFrame(RTMPContentType FrameType, RawData* Pkt, uint32 TimestampMs);
};

#endif // WITH_IBMRTMPINGEST

