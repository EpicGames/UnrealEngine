// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISubmixBufferListener.h"

#include "Logging/LogMacros.h"
#include "AudioMixerDevice.h"

#include "RHI.h"
#include "RHIResources.h"

#include "HAL/Thread.h"

THIRD_PARTY_INCLUDES_START
#pragma warning(push)
#pragma warning(disable: 4596 6319 6323)

#pragma warning(pop)
THIRD_PARTY_INCLUDES_END

#include "AudioEncoder.h"
#include "VideoEncoder.h"
#include "MediaPacket.h"
#include "Templates/SharedPointer.h"
#include "Templates/SharedPointerInternals.h"
#include "Slate/SlateViewportProvider.h"
#include "VideoEncoderInput.h"

class SWindow;

class IGameplayMediaEncoderListener
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void OnMediaSample(const AVEncoder::FMediaPacket& Sample) = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class UE_DEPRECATED(5.4, "GameplayMediaEncoder will move to a plugin in a future Unreal Engine release.") FGameplayMediaEncoder final : public ISubmixBufferListener, public AVEncoder::IAudioEncoderListener
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:

	/**
	 * Get the singleton
	 */
	static GAMEPLAYMEDIAENCODER_API FGameplayMediaEncoder* Get();

	GAMEPLAYMEDIAENCODER_API ~FGameplayMediaEncoder();

	GAMEPLAYMEDIAENCODER_API bool RegisterListener(IGameplayMediaEncoderListener* Listener);
	GAMEPLAYMEDIAENCODER_API void UnregisterListener(IGameplayMediaEncoderListener* Listener);

	GAMEPLAYMEDIAENCODER_API void SetVideoBitrate(uint32 Bitrate);
	GAMEPLAYMEDIAENCODER_API void SetVideoFramerate(uint32 Framerate);

	///**
	// * Returns the audio codec name and configuration
	// */
	//TPair<FString, AVEncoder::FAudioConfig> GetAudioConfig() const;
	//TPair<FString, AVEncoder::FVideoConfig> GetVideoConfig() const;

	GAMEPLAYMEDIAENCODER_API bool Initialize();
	GAMEPLAYMEDIAENCODER_API void Shutdown();
	GAMEPLAYMEDIAENCODER_API bool Start();
	GAMEPLAYMEDIAENCODER_API void Stop();

	static void InitializeCmd()
	{
		// We call Get(), so it creates the singleton
		Get()->Initialize();
	}

	static void ShutdownCmd()
	{
		// We call Get(), so it creates the singleton
		Get()->Shutdown();
	}

	static void StartCmd()
	{
		// We call Get(), so it creates the singleton
		Get()->Start();
	}

	static void StopCmd()
	{
		// We call Get(), so it creates the singleton
		Get()->Stop();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GAMEPLAYMEDIAENCODER_API AVEncoder::FAudioConfig GetAudioConfig() const;
	AVEncoder::FVideoConfig GetVideoConfig() const { return VideoConfig; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	// Private to control how our single instance is created
	GAMEPLAYMEDIAENCODER_API FGameplayMediaEncoder();

	// Returns how long it has been recording for.
	GAMEPLAYMEDIAENCODER_API FTimespan GetMediaTimestamp() const;

	// Back buffer capture
	GAMEPLAYMEDIAENCODER_API void OnFrameBufferReady(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider);

	// ISubmixBufferListener interface
	GAMEPLAYMEDIAENCODER_API const FString& GetListenerName() const override;
	GAMEPLAYMEDIAENCODER_API void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock) override;

	GAMEPLAYMEDIAENCODER_API void ProcessAudioFrame(const float* AudioData, int32 NumSamples, int32 NumChannels, int32 SampleRate);
	GAMEPLAYMEDIAENCODER_API void ProcessVideoFrame(const FTextureRHIRef& FrameBuffer);

	GAMEPLAYMEDIAENCODER_API void UpdateVideoConfig();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GAMEPLAYMEDIAENCODER_API void OnEncodedAudioFrame(const AVEncoder::FMediaPacket& Packet) override;
	GAMEPLAYMEDIAENCODER_API void OnEncodedVideoFrame(uint32 LayerIndex, const TSharedPtr<AVEncoder::FVideoEncoderInputFrame> Frame, const AVEncoder::FCodecPacket& Packet);

	GAMEPLAYMEDIAENCODER_API TSharedPtr<AVEncoder::FVideoEncoderInputFrame> ObtainInputFrame();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	GAMEPLAYMEDIAENCODER_API void CopyTexture(const FTextureRHIRef& SourceTexture, FTextureRHIRef& DestinationTexture) const;

	GAMEPLAYMEDIAENCODER_API void FloatToPCM16(float const* floatSamples, int32 numSamples, TArray<int16>& out) const;

	FCriticalSection ListenersCS;
	TArray<IGameplayMediaEncoderListener*> Listeners;

	FCriticalSection AudioProcessingCS;
	FCriticalSection VideoProcessingCS;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TUniquePtr<AVEncoder::FAudioEncoder> AudioEncoder;

	AVEncoder::FVideoConfig VideoConfig;

	TUniquePtr<AVEncoder::FVideoEncoder> VideoEncoder;
	TSharedPtr<AVEncoder::FVideoEncoderInput> VideoEncoderInput;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	uint64 NumCapturedFrames = 0;
	FTimespan StartTime = 0;

	// Instead of using the AudioClock parameter ISubmixBufferListener::OnNewSubmixBuffer gives us, we calculate our own, by
	// advancing it as we receive more data.
	// This is so that we can adjust the clock if things get out of sync, such as if we break into the debugger.
	double AudioClock = 0;

	FTimespan LastVideoInputTimestamp = 0;

	bool bAudioFormatChecked = false;
	bool bDoFrameSkipping = false;

	friend class FGameplayMediaEncoderModule;
	static GAMEPLAYMEDIAENCODER_API TSharedPtr<FGameplayMediaEncoder, ESPMode::ThreadSafe> Singleton;

	// live streaming: quality adaptation to available uplink b/w
	TAtomic<uint32> NewVideoBitrate{ 0 };
	FThreadSafeBool bChangeBitrate = false;
	TAtomic<uint32> NewVideoFramerate{ 0 };
	FThreadSafeBool bChangeFramerate = false;

	TArray<int16> PCM16;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TMap<TSharedPtr<AVEncoder::FVideoEncoderInputFrame>, FTextureRHIRef> BackBuffers;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool bIsInDestructor = false;
};


